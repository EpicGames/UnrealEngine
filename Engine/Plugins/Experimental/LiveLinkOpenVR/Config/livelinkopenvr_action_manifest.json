{
	"default_bindings": [
		{
			"controller_type": "knuckles",
			"binding_url": "livelinkopenvr_bindings_knuckles.json"
		}
	],
	"actions": [
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftAnalog_2D",
			"type": "vector2"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightAnalog_2D",
			"type": "vector2"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftTriggerAnalog",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightTriggerAnalog",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftThumb",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightThumb",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/SpecialLeft",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/SpecialLeft_2D",
			"type": "vector2"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/SpecialRight",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonBottom",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonRight",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonLeft",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonTop",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftShoulder",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightShoulder",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftTriggerThreshold",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightTriggerThreshold",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/DPadUp",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/DPadDown",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/DPadRight",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/DPadLeft",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftStick_2D",
			"type": "vector2"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftStickDown",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/LeftStickLeft",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightStick_2D",
			"type": "vector2"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightStickDown",
			"type": "vector1"
		},
		{
			"name": "/actions/LiveLinkGamepadInputDevice/in/RightStickLeft",
			"type": "vector1"
		}
	],
	"localization": [
		{
			"language_tag": "en_US"
		}
	]
}
