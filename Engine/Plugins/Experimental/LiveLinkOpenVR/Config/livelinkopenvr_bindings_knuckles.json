{
	"controller_type": "knuckles",
	"name": "LiveLinkOpenVR bindings for Valve Index controller",
	"description": "Bindings for the Live Link OpenVR plugin for the Valve Index controller",
	"bindings": {
		"/actions/LiveLinkGamepadInputDevice": {
			"sources": [
				{
					"path": "/user/hand/left/input/thumbstick",
					"mode": "joystick",
					"inputs": {
						"position": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/LeftStick_2D"
						},
						"click": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/LeftThumb"
						}
					}
				},
				{
					"path": "/user/hand/left/input/trigger",
					"mode": "trigger",
					"inputs": {
						"pull": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/LeftTriggerAnalog"
						}
					}
				},
				{
					"path": "/user/hand/left/input/trackpad",
					"mode": "trackpad",
					"inputs": {
						"position": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/LeftAnalog_2D"
						}
					}
				},
				{
					"path": "/user/hand/left/input/trackpad",
					"mode": "force_sensor",
					"inputs": {
						"force": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/SpecialLeft"
						}
					}
				},
				{
					"path": "/user/hand/left/input/a",
					"mode": "button",
					"inputs": {
						"click": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonLeft"
						}
					}
				},
				{
					"path": "/user/hand/left/input/b",
					"mode": "button",
					"inputs": {
						"click": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonTop"
						}
					}
				},
				{
					"path": "/user/hand/left/input/grip",
					"mode": "trigger",
					"inputs": {
						"pull": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/LeftShoulder"
						}
					}
				},
				{
					"path": "/user/hand/right/input/thumbstick",
					"mode": "joystick",
					"inputs": {
						"position": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/RightStick_2D"
						},
						"click": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/RightThumb"
						}
					}
				},
				{
					"path": "/user/hand/right/input/trigger",
					"mode": "trigger",
					"inputs": {
						"pull": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/RightTriggerAnalog"
						}
					}
				},
				{
					"path": "/user/hand/right/input/trackpad",
					"mode": "trackpad",
					"inputs": {
						"position": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/RightAnalog_2D"
						}
					}
				},
				{
					"path": "/user/hand/right/input/trackpad",
					"mode": "force_sensor",
					"inputs": {
						"force": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/SpecialRight"
						}
					}
				},
				{
					"path": "/user/hand/right/input/a",
					"mode": "button",
					"inputs": {
						"click": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonBottom"
						}
					}
				},
				{
					"path": "/user/hand/right/input/b",
					"mode": "button",
					"inputs": {
						"click": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/FaceButtonRight"
						}
					}
				},
				{
					"path": "/user/hand/right/input/grip",
					"mode": "trigger",
					"inputs": {
						"pull": {
							"output": "/actions/LiveLinkGamepadInputDevice/in/RightShoulder"
						}
					}
				}
			]
		}
	}
}
