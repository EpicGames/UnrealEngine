{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkOpenVR",
	"Description": "Live Link plugin for OpenVR (Not supported for native arm64.)",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"EnabledByDefault": false,
	"SupportedTargetPlatforms": [
		"Win64"
	],
	"Modules": [
		{
			"Name": "LiveLinkOpenVR",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [
				"Win64"
			],
			"PlatformArchitectureDenyList": [
				"Win64:arm64"
			]
		}
	]
}
