// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class LiveLinkOpenVR : ModuleRules
	{
		public LiveLinkOpenVR(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"LiveLinkInterface",
					"Projects",
					"Slate",
					"SlateCore",

					"OpenVR",
				});
		}
	}
}
