// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenVRModule.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Interfaces/IPluginManager.h"
#include "Logging/StructuredLog.h"
#include "Misc/Paths.h"

#include <openvr.h>


DEFINE_LOG_CATEGORY(LogLiveLinkOpenVR);

IMPLEMENT_MODULE(FLiveLinkOpenVRModule, LiveLinkOpenVR);


void FLiveLinkOpenVRModule::StartupModule()
{
}


void FLiveLinkOpenVRModule::ShutdownModule()
{
	if (VrSystem)
	{
		vr::VR_Shutdown();
		VrSystem = nullptr;
	}

	UnloadOpenVRLibrary();
}


vr::IVRSystem* FLiveLinkOpenVRModule::GetVrSystem()
{
	if (!VrSystem)
	{
		ensure(LoadOpenVRLibrary());
	}

	return VrSystem;
}


bool FLiveLinkOpenVRModule::LoadOpenVRLibrary()
{
	const FString PluginBaseDir = IPluginManager::Get().FindPlugin(TEXT("LiveLinkOpenVR"))->GetBaseDir();

	const TCHAR* const OpenVRSdkVer = TEXT("OpenVRv1_5_17");
	FString OpenVRSdkRoot = FString::Printf(TEXT("%s/Source/ThirdParty/OpenVR/%s"), *PluginBaseDir, OpenVRSdkVer);

#if PLATFORM_WINDOWS
	FString VROverridePath = FPlatformMisc::GetEnvironmentVariable(TEXT("VR_OVERRIDE"));
	if (VROverridePath.Len() > 0)
	{
		OpenVRSdkRoot = VROverridePath;
	}

	const FString OpenVRDLLDir = FPaths::Combine(OpenVRSdkRoot, "bin", "win64");
	FPlatformProcess::PushDllDirectory(*OpenVRDLLDir);
	OpenVRDLLHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(OpenVRDLLDir, "openvr_api.dll"));
	FPlatformProcess::PopDllDirectory(*OpenVRDLLDir);
#elif PLATFORM_MAC
	const FString OpenVRDLLDir = FPaths::Combine(OpenVRSdkRoot, "bin", "osx32");
	OpenVRDLLHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(OpenVRDllDir, "libopenvr_api.dylib"));
#elif PLATFORM_LINUX
	const FString OpenVRDllDir = FPaths::Combine(OpenVRSdkRoot, "bin", "linux64");
	OpenVRDLLHandle = FPlatformProcess::GetDllHandle(*FPaths::Combine(OpenVRDllDir, "libopenvr_api.so"));
#else
#error "OpenVR is not supported for this platform."
#endif

	if (!OpenVRDLLHandle)
	{
		UE_LOGFMT(LogLiveLinkOpenVR, Log, "Failed to load OpenVR library.");
		return false;
	}

	vr::EVRInitError VrInitError = vr::VRInitError_None;
	VrSystem = vr::VR_Init(&VrInitError, vr::VRApplication_Other);

	if (VrInitError != vr::VRInitError_None)
	{
		VrSystem = nullptr;
		UnloadOpenVRLibrary();
		return false;
	}

	FString ManifestPath = FPaths::Combine(PluginBaseDir, "Config", "livelinkopenvr_action_manifest.json");
	ManifestPath = IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*ManifestPath);
	const vr::EVRInputError InputError = vr::VRInput()->SetActionManifestPath(TCHAR_TO_UTF8(*ManifestPath));
	if (InputError != vr::EVRInputError::VRInputError_None)
	{
		UE_LOGFMT(LogLiveLinkOpenVR, Error, "IVRInput::SetActionManifestPath failed with result {InputError}", InputError);
	}

	return true;
}


bool FLiveLinkOpenVRModule::UnloadOpenVRLibrary()
{
	if (VrSystem)
	{
		vr::VR_Shutdown();
		VrSystem = nullptr;
	}

	if (OpenVRDLLHandle)
	{
		FPlatformProcess::FreeDllHandle(OpenVRDLLHandle);
		OpenVRDLLHandle = nullptr;
	}

	return true;
}
