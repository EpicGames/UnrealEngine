// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

namespace vr
{
	class IVRSystem;
}


DECLARE_LOG_CATEGORY_EXTERN(LogLiveLinkOpenVR, Log, All);


class FLiveLinkOpenVRModule : public IModuleInterface
{
public:
	static FLiveLinkOpenVRModule& Get()
	{
		return FModuleManager::Get().LoadModuleChecked<FLiveLinkOpenVRModule>(TEXT("LiveLinkOpenVR"));
	}

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	vr::IVRSystem* GetVrSystem();

private:
	bool LoadOpenVRLibrary();
	bool UnloadOpenVRLibrary();

private:
	void* OpenVRDLLHandle = nullptr;
	vr::IVRSystem* VrSystem = nullptr;
};
