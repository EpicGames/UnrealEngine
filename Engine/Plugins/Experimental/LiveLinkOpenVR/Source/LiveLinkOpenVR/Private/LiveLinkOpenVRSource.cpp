// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenVRSource.h"
#include "HAL/RunnableThread.h"
#include "ILiveLinkClient.h"
#include "LiveLinkOpenVRModule.h"
#include "LiveLinkSubjectSettings.h"
#include "Logging/StructuredLog.h"
#include "Misc/App.h"
#include "Misc/CoreDelegates.h"
#include "Roles/LiveLinkInputDeviceRole.h"
#include "Roles/LiveLinkInputDeviceTypes.h"
#include "Roles/LiveLinkTransformRole.h"
#include "Roles/LiveLinkTransformTypes.h"

#include <openvr.h>


#define LOCTEXT_NAMESPACE "LiveLinkOpenVR"


namespace
{
	FMatrix ToFMatrix(const vr::HmdMatrix34_t& tm)
	{
		// Rows and columns are swapped between vr::HmdMatrix34_t and FMatrix
		return FMatrix(
			FPlane(tm.m[0][0], tm.m[1][0], tm.m[2][0], 0.0f),
			FPlane(tm.m[0][1], tm.m[1][1], tm.m[2][1], 0.0f),
			FPlane(tm.m[0][2], tm.m[1][2], tm.m[2][2], 0.0f),
			FPlane(tm.m[0][3], tm.m[1][3], tm.m[2][3], 1.0f)
		);
	}

	FMatrix ToFMatrix(const vr::HmdMatrix44_t& tm)
	{
		// Rows and columns are swapped between vr::HmdMatrix44_t and FMatrix
		return FMatrix(
			FPlane(tm.m[0][0], tm.m[1][0], tm.m[2][0], tm.m[3][0]),
			FPlane(tm.m[0][1], tm.m[1][1], tm.m[2][1], tm.m[3][1]),
			FPlane(tm.m[0][2], tm.m[1][2], tm.m[2][2], tm.m[3][2]),
			FPlane(tm.m[0][3], tm.m[1][3], tm.m[2][3], tm.m[3][3])
		);
	}
} // namespace


FLiveLinkOpenVRSource::FLiveLinkOpenVRSource(const FLiveLinkOpenVRConnectionSettings& InConnectionSettings)
	: ConnectionSettings(InConnectionSettings)
	, SubjectNames(NAME_None)
{
	SourceStatus = LOCTEXT("SourceStatus_NoData", "No data");
	SourceType = LOCTEXT("SourceType_OpenVR", "OpenVR");
	SourceMachineName = LOCTEXT("Source_MachineName", "Local OpenVR");

	FLiveLinkOpenVRModule& Module = FLiveLinkOpenVRModule::Get();
	vr::IVRSystem* VrSystem = Module.GetVrSystem();
	if (!VrSystem)
	{
		UE_LOGF(LogLiveLinkOpenVR, Error, "LiveLinkOpenVRSource: Couldn't get IVRSystem");
		return;
	}

	DeferredStartDelegateHandle = FCoreDelegates::OnEndFrame.AddRaw(this, &FLiveLinkOpenVRSource::Start);
}


FLiveLinkOpenVRSource::~FLiveLinkOpenVRSource()
{
	// This could happen if the object is destroyed before FCoreDelegates::OnEndFrame calls FLiveLinkOpenVRSource::Start
	if (DeferredStartDelegateHandle.IsValid())
	{
		FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	}

	if (Client)
	{
		Client->OnLiveLinkSubjectAdded().Remove(OnSubjectAddedDelegate);
	}

	Stop();

	if (Thread != nullptr)
	{
		Thread->WaitForCompletion();
		delete Thread;
		Thread = nullptr;
	}
}


void FLiveLinkOpenVRSource::ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid)
{
	Client = InClient;
	SourceGuid = InSourceGuid;

	OnSubjectAddedDelegate = Client->OnLiveLinkSubjectAdded().AddRaw(this, &FLiveLinkOpenVRSource::OnLiveLinkSubjectAdded);
}


void FLiveLinkOpenVRSource::InitializeSettings(ULiveLinkSourceSettings* InSettings)
{
	ULiveLinkOpenVRSourceSettings* SourceSettings = Cast<ULiveLinkOpenVRSourceSettings>(InSettings);
	if (!ensure(SourceSettings))
	{
		return;
	}

	LocalUpdateRateInHz_AnyThread = SourceSettings->CommonSettings.LocalUpdateRateInHz;
}


bool FLiveLinkOpenVRSource::IsSourceStillValid() const
{
	// Source is valid if we have a valid thread
	const bool bIsSourceValid = !bStopping && (Thread != nullptr);
	return bIsSourceValid;
}


bool FLiveLinkOpenVRSource::RequestSourceShutdown()
{
	Stop();

	return true;
}


TSubclassOf<ULiveLinkSourceSettings> FLiveLinkOpenVRSource::GetSettingsClass() const
{
	return ULiveLinkOpenVRSourceSettings::StaticClass();
}


void FLiveLinkOpenVRSource::Start()
{
	check(DeferredStartDelegateHandle.IsValid());

	FCoreDelegates::OnEndFrame.Remove(DeferredStartDelegateHandle);
	DeferredStartDelegateHandle.Reset();
	
	SourceStatus = LOCTEXT("SourceStatus_Receiving", "Receiving");

	static std::atomic<int32> ReceiverIndex = 0;
	ThreadName = "LiveLinkOpenVR Receiver ";
	ThreadName.AppendInt(++ReceiverIndex);

	Thread = FRunnableThread::Create(this, *ThreadName, 128 * 1024, TPri_AboveNormal, FPlatformAffinity::GetPoolThreadMask());
}


void FLiveLinkOpenVRSource::Stop()
{
	bStopping = true;
}


struct FOpenVRInputAction
{
	enum EActionType
	{
		Digital,
		Analog1D,
		Analog2D,
	};

	FName ActionName;
	EActionType ActionType;
	TOptional<float FLiveLinkGamepadInputDeviceFrameData::*> OutputFieldX;
	TOptional<float FLiveLinkGamepadInputDeviceFrameData::*> OutputFieldY;

	vr::VRActionHandle_t Handle = vr::k_ulInvalidActionHandle;

	union {
		vr::InputDigitalActionData_t Digital;
		vr::InputAnalogActionData_t Analog;
	} LastActionData;
};


uint32 FLiveLinkOpenVRSource::Run()
{
	TArray<FOpenVRInputAction> Actions = {
		{ "LeftAnalog_2D", FOpenVRInputAction::Analog2D,
		  &FLiveLinkGamepadInputDeviceFrameData::LeftAnalogX, &FLiveLinkGamepadInputDeviceFrameData::LeftAnalogY },

		{ "RightAnalog_2D", FOpenVRInputAction::Analog2D,
		  &FLiveLinkGamepadInputDeviceFrameData::RightAnalogX, &FLiveLinkGamepadInputDeviceFrameData::RightAnalogY },

		{ "SpecialLeft_2D", FOpenVRInputAction::Analog2D,
		  &FLiveLinkGamepadInputDeviceFrameData::SpecialLeft_X, &FLiveLinkGamepadInputDeviceFrameData::SpecialLeft_Y },

		{ "LeftStick_2D", FOpenVRInputAction::Analog2D,
		  &FLiveLinkGamepadInputDeviceFrameData::LeftStickRight, &FLiveLinkGamepadInputDeviceFrameData::LeftStickUp },

		{ "RightStick_2D", FOpenVRInputAction::Analog2D,
		  &FLiveLinkGamepadInputDeviceFrameData::RightStickRight, &FLiveLinkGamepadInputDeviceFrameData::RightStickUp },

		// All actions delcared in the manifest are listed below.
		// Most are 1D/float actions, but the few that are commented
		// out are the ones that have been combined into 2D axes above.

#define DECLARE_VECTOR1_ACTION(ActionNameToken)                        \
		{ UE_STRINGIZE(ActionNameToken), FOpenVRInputAction::Analog1D, \
		  &FLiveLinkGamepadInputDeviceFrameData::ActionNameToken },

		//DECLARE_VECTOR1_ACTION(LeftAnalogX)
		//DECLARE_VECTOR1_ACTION(LeftAnalogY)
		//DECLARE_VECTOR1_ACTION(RightAnalogX)
		//DECLARE_VECTOR1_ACTION(RightAnalogY)
		DECLARE_VECTOR1_ACTION(LeftTriggerAnalog)
		DECLARE_VECTOR1_ACTION(RightTriggerAnalog)
		DECLARE_VECTOR1_ACTION(LeftThumb)
		DECLARE_VECTOR1_ACTION(RightThumb)
		DECLARE_VECTOR1_ACTION(SpecialLeft)
		//DECLARE_VECTOR1_ACTION(SpecialLeft_X)
		//DECLARE_VECTOR1_ACTION(SpecialLeft_Y)
		DECLARE_VECTOR1_ACTION(SpecialRight)
		DECLARE_VECTOR1_ACTION(FaceButtonBottom)
		DECLARE_VECTOR1_ACTION(FaceButtonRight)
		DECLARE_VECTOR1_ACTION(FaceButtonLeft)
		DECLARE_VECTOR1_ACTION(FaceButtonTop)
		DECLARE_VECTOR1_ACTION(LeftShoulder)
		DECLARE_VECTOR1_ACTION(RightShoulder)
		DECLARE_VECTOR1_ACTION(LeftTriggerThreshold)
		DECLARE_VECTOR1_ACTION(RightTriggerThreshold)
		DECLARE_VECTOR1_ACTION(DPadUp)
		DECLARE_VECTOR1_ACTION(DPadDown)
		DECLARE_VECTOR1_ACTION(DPadRight)
		DECLARE_VECTOR1_ACTION(DPadLeft)
		//DECLARE_VECTOR1_ACTION(LeftStickUp)
		DECLARE_VECTOR1_ACTION(LeftStickDown)
		//DECLARE_VECTOR1_ACTION(LeftStickRight)
		DECLARE_VECTOR1_ACTION(LeftStickLeft)
		//DECLARE_VECTOR1_ACTION(RightStickUp)
		DECLARE_VECTOR1_ACTION(RightStickDown)
		//DECLARE_VECTOR1_ACTION(RightStickRight)
		DECLARE_VECTOR1_ACTION(RightStickLeft)

#undef DECLARE_VECTOR1_ACTION
	};

	FLiveLinkOpenVRModule& Module = FLiveLinkOpenVRModule::Get();
	vr::IVRSystem* VrSystem = Module.GetVrSystem();
	vr::IVRInput* VrInput = vr::VRInput();

	const char* const ActionSetPath = "/actions/LiveLinkGamepadInputDevice";
	vr::VRActionSetHandle_t ActionSet = vr::k_ulInvalidActionSetHandle;
	vr::EVRInputError InputError = VrInput->GetActionSetHandle(ActionSetPath, &ActionSet);
	if (InputError != vr::VRInputError_None)
	{
		UE_LOGFMT(LogLiveLinkOpenVR, Error, "IVRInput::GetActionSetHandle failed with result {InputError}", InputError);
		ActionSet = vr::k_ulInvalidActionSetHandle;
	}
	else
	{
		for (FOpenVRInputAction& Action : Actions)
		{
			constexpr int32 MaxPathLen = 256;
			TAnsiStringBuilder<MaxPathLen> ActionPath;
			ActionPath.Appendf("%s/in/%S", ActionSetPath, *Action.ActionName.ToString());
			InputError = VrInput->GetActionHandle(*ActionPath, &Action.Handle);
			if (InputError != vr::VRInputError_None)
			{
				UE_LOGFMT(LogLiveLinkOpenVR, Error, "IVRInput::GetActionHandle for '{ActionPath}' failed with result {InputError}", *ActionPath, InputError);
				Action.Handle = vr::k_ulInvalidActionHandle;
			}
		}
	}

	static const FName InputSubjectName("OpenVRInput");
	Client->PushSubjectStaticData_AnyThread({ SourceGuid, InputSubjectName }, ULiveLinkInputDeviceRole::StaticClass(),
		FLiveLinkStaticDataStruct(FLiveLinkGamepadInputDeviceStaticData::StaticStruct()));

	TStaticArray<vr::TrackedDevicePose_t, vr::k_unMaxTrackedDeviceCount> Poses;
	TStringBuilder<256> StringBuilder;

	double LastFrameTimeSec = -DBL_MAX;
	while (!bStopping)
	{
		// Send new poses at the user specified update rate
		const double FrameIntervalSec = 1.0 / LocalUpdateRateInHz_AnyThread;
		const double TimeNowSec = FPlatformTime::Seconds();
		if (TimeNowSec >= (LastFrameTimeSec + FrameIntervalSec))
		{
			LastFrameTimeSec = TimeNowSec;

			const TOptional<FQualifiedFrameTime> CurrentFrameTime = FApp::GetCurrentFrameTime();

			// Update poses.
			VrSystem->GetDeviceToAbsoluteTrackingPose(
				vr::ETrackingUniverseOrigin::TrackingUniverseStanding,
				0.0f,
				Poses.GetData(),
				Poses.Num()
			);

			// Update inputs.
			vr::VRActiveActionSet_t ActiveSet = { 0 };
			ActiveSet.ulActionSet = ActionSet;
			VrInput->UpdateActionState(&ActiveSet, sizeof(ActiveSet), 1);

			// Enumerate poses.
			for (int32 DeviceIdx = 0; DeviceIdx < Poses.Num(); ++DeviceIdx)
			{
				const vr::TrackedDevicePose_t& Pose = Poses[DeviceIdx];
				if (!Pose.bDeviceIsConnected)
				{
					continue;
				}

				FName SubjectName = SubjectNames[DeviceIdx];

				// If we don't have a name, it's a new subject.
				if (SubjectName == NAME_None)
				{
					StringBuilder.Reset();

					const vr::ETrackedDeviceClass DeviceClass = VrSystem->GetTrackedDeviceClass(DeviceIdx);
					switch (DeviceClass)
					{
						case vr::TrackedDeviceClass_HMD:
							if (!ConnectionSettings.bTrackHMDs)
							{
								continue;
							}

							StringBuilder << TEXT("HMD");
							break;

						case vr::TrackedDeviceClass_Controller:
							if (!ConnectionSettings.bTrackControllers)
							{
								continue;
							}

							StringBuilder << TEXT("Controller");
							break;

						case vr::TrackedDeviceClass_GenericTracker:
							if (!ConnectionSettings.bTrackTrackers)
							{
								continue;
							}

							StringBuilder << TEXT("Tracker");
							break;

						case vr::TrackedDeviceClass_TrackingReference:
							if (!ConnectionSettings.bTrackTrackingReferences)
							{
								continue;
							}

							StringBuilder << TEXT("TrackingRef");
							break;

						default:
							StringBuilder << TEXT("Other");
							break;
					}

					StringBuilder << TEXT("_");

					char SerialNumBuf[128] = { 0 };
					VrSystem->GetStringTrackedDeviceProperty(DeviceIdx, vr::Prop_SerialNumber_String, SerialNumBuf, sizeof(SerialNumBuf));
					StringBuilder << SerialNumBuf;

					SubjectName = SubjectNames[DeviceIdx] = FName(StringBuilder.ToString());

					// If the LiveLink client already knows about this subject, then it must have been added via a preset
					// Only new subjects should be set to rebroadcast by default. Presets should respect the existing settings
					if (!Client->GetSubjects(true, true).Contains(FLiveLinkSubjectKey(SourceGuid, SubjectName)))
					{
						SubjectsToRebroadcast.Add(SubjectName);
					}

					FLiveLinkStaticDataStruct StaticData(FLiveLinkTransformStaticData::StaticStruct());
					Client->PushSubjectStaticData_AnyThread({SourceGuid, SubjectName}, ULiveLinkTransformRole::StaticClass(), MoveTemp(StaticData));
				}

				// Send transform frame data, if available.
				if (Pose.bPoseIsValid)
				{
					// Transpose and decompose.
					const FMatrix PoseMatrix = ToFMatrix(Pose.mDeviceToAbsoluteTracking);
					const FQuat PoseOrientation(PoseMatrix);
					const FVector PosePosition(PoseMatrix.M[3][0], PoseMatrix.M[3][1], PoseMatrix.M[3][2]);

					// Handedness/basis change + scale.
					const double MetersToUnrealUnits = 100.0; // cm
					const FTransform PoseTransform(
						FQuat(-PoseOrientation.Z, PoseOrientation.X, PoseOrientation.Y, -PoseOrientation.W),
						FVector(-PosePosition.Z, PosePosition.X, PosePosition.Y) * MetersToUnrealUnits
					);

					FLiveLinkFrameDataStruct TransformStruct(FLiveLinkTransformFrameData::StaticStruct());
					FLiveLinkTransformFrameData* TransformFrameData = TransformStruct.Cast<FLiveLinkTransformFrameData>();

					TransformFrameData->WorldTime = TimeNowSec;
					if (CurrentFrameTime)
					{
						TransformFrameData->MetaData.SceneTime = *CurrentFrameTime;
					}

					TransformFrameData->Transform = PoseTransform;

					Send(MoveTemp(TransformStruct), SubjectName);
				}
			}

			// Enumerate actions.
			FLiveLinkFrameDataStruct InputStruct(FLiveLinkGamepadInputDeviceFrameData::StaticStruct());
			FLiveLinkGamepadInputDeviceFrameData* InputFrameData = InputStruct.Cast<FLiveLinkGamepadInputDeviceFrameData>();

			InputFrameData->WorldTime = TimeNowSec;
			if (CurrentFrameTime)
			{
				InputFrameData->MetaData.SceneTime = *CurrentFrameTime;
			}

			for (FOpenVRInputAction& Action : Actions)
			{
				// Populate action data.
				switch (Action.ActionType)
				{
					case FOpenVRInputAction::Digital:
					{
						const vr::VRInputValueHandle_t Unrestricted = vr::k_ulInvalidInputValueHandle;
						InputError = VrInput->GetDigitalActionData(Action.Handle,
							&Action.LastActionData.Digital,
							sizeof(Action.LastActionData.Digital),
							Unrestricted);
						break;
					}

					case FOpenVRInputAction::Analog1D:
					case FOpenVRInputAction::Analog2D:
					{
						const vr::VRInputValueHandle_t Unrestricted = vr::k_ulInvalidInputValueHandle;
						InputError = VrInput->GetAnalogActionData(Action.Handle,
							&Action.LastActionData.Analog,
							sizeof(Action.LastActionData.Analog),
							Unrestricted);
						break;
					}

					default:
						checkNoEntry();
				}

				if (InputError != vr::VRInputError_None)
				{
					UE_LOGFMT(LogLiveLinkOpenVR, Error,
						"IVRInput::Get*ActionData for '{ActionName}' failed with result {InputError}",
						Action.ActionName, InputError);
					continue;
				}

				switch (Action.ActionType)
				{
					case FOpenVRInputAction::Digital:
					{
						float& DestField = InputFrameData->**Action.OutputFieldX;
						DestField = Action.LastActionData.Digital.bState ? 1.0f : 0.0f;
						break;
					}

					case FOpenVRInputAction::Analog1D:
					{
						float& DestFieldX = InputFrameData->**Action.OutputFieldX;
						DestFieldX = Action.LastActionData.Analog.x;
						break;
					}

					case FOpenVRInputAction::Analog2D:
					{
						float& DestFieldX = InputFrameData->**Action.OutputFieldX;
						float& DestFieldY = InputFrameData->**Action.OutputFieldY;
						DestFieldX = Action.LastActionData.Analog.x;
						DestFieldY = Action.LastActionData.Analog.y;
						break;
					}

					default:
						checkNoEntry();
				}
			}

			Send(MoveTemp(InputStruct), InputSubjectName);
		}

		FPlatformProcess::Sleep(0.001f);
	}
	
	return 0;
}


void FLiveLinkOpenVRSource::Send(FLiveLinkFrameDataStruct&& InFrameData, FName InSubjectName)
{
	if (bStopping || (Client == nullptr))
	{
		return;
	}

	Client->PushSubjectFrameData_AnyThread({ SourceGuid, InSubjectName }, MoveTemp(InFrameData));
}


void FLiveLinkOpenVRSource::OnLiveLinkSubjectAdded(FLiveLinkSubjectKey InSubjectKey)
{
	// Set rebroadcast to true for any new subjects
	if (SubjectsToRebroadcast.Contains(InSubjectKey.SubjectName))
	{
		ULiveLinkSubjectSettings* SubjectSettings = Cast<ULiveLinkSubjectSettings>(Client->GetSubjectSettings(InSubjectKey));
		if (SubjectSettings)
		{
			SubjectSettings->bRebroadcastSubject = true;
		}
	}
}


#undef LOCTEXT_NAMESPACE
