// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StaticArray.h"
#include "HAL/Runnable.h"
#include "ILiveLinkSource.h"
#include "LiveLinkOpenVRTypes.h"


class FLiveLinkOpenVRSource
	: public ILiveLinkSource
	, public FRunnable
	, public TSharedFromThis<FLiveLinkOpenVRSource>
{
public:
	FLiveLinkOpenVRSource(const FLiveLinkOpenVRConnectionSettings& InConnectionSettings);

	virtual ~FLiveLinkOpenVRSource();

	//~ Begin ILiveLinkSource interface
	virtual void ReceiveClient(ILiveLinkClient* InClient, FGuid InSourceGuid) override;
	virtual void InitializeSettings(ULiveLinkSourceSettings* InSettings) override;
	virtual void Update() override { }

	virtual bool IsSourceStillValid() const override;

	virtual bool RequestSourceShutdown() override;

	virtual FText GetSourceType() const override { return SourceType; };
	virtual FText GetSourceMachineName() const override { return SourceMachineName; }
	virtual FText GetSourceStatus() const override { return SourceStatus; }

	virtual TSubclassOf<ULiveLinkSourceSettings> GetSettingsClass() const override;
	//~ End ILiveLinkSource interface

	//~ Begin FRunnable interface
	virtual bool Init() override { return true; }
	virtual uint32 Run() override;
	void Start();
	virtual void Stop() override;
	virtual void Exit() override { }
	//~ End FRunnable interface

	void Send(FLiveLinkFrameDataStruct&& InFrameData, FName InSubjectName);

private:
	// Callback when the a livelink subject has been added
	void OnLiveLinkSubjectAdded(FLiveLinkSubjectKey InSubjectKey);

private:
	const FLiveLinkOpenVRConnectionSettings ConnectionSettings;
	ILiveLinkClient* Client = nullptr;

	// Our identifier in LiveLink
	FGuid SourceGuid;

	FText SourceType;
	FText SourceMachineName;
	FText SourceStatus;

	// Threadsafe flag for terminating the main thread loop
	std::atomic<bool> bStopping = false;

	// Thread to update poses from
	FRunnableThread* Thread = nullptr;

	// Name of the update thread
	FString ThreadName;

	// List of subjects we've already encountered
	TStaticArray<FName, 64> SubjectNames;

	// List of subjects to automatically set to rebroadcast
	TSet<FName> SubjectsToRebroadcast;

	// Deferred start delegate handle.
	FDelegateHandle DeferredStartDelegateHandle;

	// frame counter for data
	int64 FrameCounter = 0;

	TWeakObjectPtr<ULiveLinkOpenVRSourceSettings> Settings;

	// Atomic copies of UObject fields updated on change notification
	std::atomic<bool> bTrackTrackers_AnyThread;
	std::atomic<bool> bTrackControllers_AnyThread;
	std::atomic<bool> bTrackHMDs_AnyThread;
	std::atomic<uint32> LocalUpdateRateInHz_AnyThread;

	// Delegate for when the LiveLink client has ticked
	FDelegateHandle OnSubjectAddedDelegate;
};
