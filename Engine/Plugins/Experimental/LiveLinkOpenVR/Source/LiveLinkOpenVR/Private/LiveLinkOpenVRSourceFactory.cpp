// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkOpenVRSourceFactory.h"
#include "LiveLinkOpenVRSource.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SCompoundWidget.h"

#if WITH_EDITOR
#	include "IStructureDetailsView.h"
#	include "PropertyEditorModule.h"
#	include "Widgets/Input/SButton.h"
#	include "Widgets/SBoxPanel.h"
#endif


#define LOCTEXT_NAMESPACE "LiveLinkOpenVR"


DECLARE_DELEGATE_OneParam(FOnLiveLinkOpenVRConnectionSettingsAccepted, FLiveLinkOpenVRConnectionSettings);


class SLiveLinkOpenVRSourceFactory : public SCompoundWidget
{
	SLATE_BEGIN_ARGS(SLiveLinkOpenVRSourceFactory) {}
		SLATE_EVENT(FOnLiveLinkOpenVRConnectionSettingsAccepted, OnConnectionSettingsAccepted)
	SLATE_END_ARGS()

	void Construct(const FArguments& Args)
	{
#if WITH_EDITOR
		OnConnectionSettingsAccepted = Args._OnConnectionSettingsAccepted;

		FStructureDetailsViewArgs StructureViewArgs;
		FDetailsViewArgs DetailArgs;
		DetailArgs.bAllowSearch = false;
		DetailArgs.bShowScrollBar = false;

		FPropertyEditorModule& PropertyEditor = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));

		StructOnScope = MakeShared<FStructOnScope>(FLiveLinkOpenVRConnectionSettings::StaticStruct());
		CastChecked<UScriptStruct>(StructOnScope->GetStruct())->CopyScriptStruct(StructOnScope->GetStructMemory(), &ConnectionSettings);
		StructureDetailsView = PropertyEditor.CreateStructureDetailView(DetailArgs, StructureViewArgs, StructOnScope);

		ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.FillHeight(1.f)
			[
				StructureDetailsView->GetWidget().ToSharedRef()
			]
			+ SVerticalBox::Slot()
			.HAlign(HAlign_Right)
			.AutoHeight()
			[
				SNew(SButton)
				.OnClicked(this, &SLiveLinkOpenVRSourceFactory::OnSettingsAccepted)
				.Text(LOCTEXT("AddSource", "Add"))
			]
		];
#endif // #if WITH_EDITOR
	}

private:
	FLiveLinkOpenVRConnectionSettings ConnectionSettings;

#if WITH_EDITOR
	TSharedPtr<FStructOnScope> StructOnScope;
	TSharedPtr<IStructureDetailsView> StructureDetailsView;
#endif

	FReply OnSettingsAccepted()
	{
#if WITH_EDITOR
		CastChecked<UScriptStruct>(StructOnScope->GetStruct())->CopyScriptStruct(&ConnectionSettings, StructOnScope->GetStructMemory());
		OnConnectionSettingsAccepted.ExecuteIfBound(ConnectionSettings);
#endif // #if WITH_EDITOR

		return FReply::Handled();
	}

	FOnLiveLinkOpenVRConnectionSettingsAccepted OnConnectionSettingsAccepted;
};


//////////////////////////////////////////////////////////////////////////


FText ULiveLinkOpenVRSourceFactory::GetSourceDisplayName() const
{
	return LOCTEXT("SourceDisplayName", "LiveLinkOpenVR Source");	
}

FText ULiveLinkOpenVRSourceFactory::GetSourceTooltip() const
{
	return LOCTEXT("SourceTooltip", "Allows creation of multiple LiveLink sources using the OpenVR tracking system");
}

TSharedPtr<SWidget> ULiveLinkOpenVRSourceFactory::BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const
{
	return SNew(SLiveLinkOpenVRSourceFactory)
		.OnConnectionSettingsAccepted(FOnLiveLinkOpenVRConnectionSettingsAccepted::CreateUObject(
			this, &ULiveLinkOpenVRSourceFactory::CreateSourceFromSettings, InOnLiveLinkSourceCreated
		));
}

TSharedPtr<ILiveLinkSource> ULiveLinkOpenVRSourceFactory::CreateSource(const FString& ConnectionString) const
{
	FLiveLinkOpenVRConnectionSettings ConnectionSettings;
	if (!ConnectionString.IsEmpty())
	{
		FLiveLinkOpenVRConnectionSettings::StaticStruct()->ImportText(
			*ConnectionString, &ConnectionSettings, nullptr, PPF_None, GLog, TEXT("ULiveLinkOpenVRSourceFactory")
		);
	}
	return MakeShared<FLiveLinkOpenVRSource>(ConnectionSettings);
}

void ULiveLinkOpenVRSourceFactory::CreateSourceFromSettings(FLiveLinkOpenVRConnectionSettings InConnectionSettings, FOnLiveLinkSourceCreated OnSourceCreated) const
{
	FString ConnectionString;
	FLiveLinkOpenVRConnectionSettings::StaticStruct()->ExportText(
		ConnectionString, &InConnectionSettings, nullptr, nullptr, PPF_None, nullptr
	);

	TSharedPtr<FLiveLinkOpenVRSource> SharedPtr = MakeShared<FLiveLinkOpenVRSource>(InConnectionSettings);
	OnSourceCreated.ExecuteIfBound(SharedPtr, MoveTemp(ConnectionString));
}


#undef LOCTEXT_NAMESPACE
