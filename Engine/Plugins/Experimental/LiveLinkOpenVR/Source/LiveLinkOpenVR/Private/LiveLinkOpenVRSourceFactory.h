// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkOpenVRTypes.h"
#include "LiveLinkSourceFactory.h"
#include "LiveLinkOpenVRSourceFactory.generated.h"


UCLASS()
class ULiveLinkOpenVRSourceFactory : public ULiveLinkSourceFactory
{
public:
	GENERATED_BODY()

	//~ Begin ULiveLinkSourceFactory interface
	virtual FText GetSourceDisplayName() const override;
	virtual FText GetSourceTooltip() const override;

	virtual EMenuType GetMenuType() const override { return EMenuType::SubPanel; }
	virtual TSharedPtr<SWidget> BuildCreationPanel(FOnLiveLinkSourceCreated InOnLiveLinkSourceCreated) const override;
	virtual TSharedPtr<ILiveLinkSource> CreateSource(const FString& InConnectionString) const override;
	//~ End ULiveLinkSourceFactory interface

private:
	void CreateSourceFromSettings(FLiveLinkOpenVRConnectionSettings InConnectionSettings, FOnLiveLinkSourceCreated InOnSourceCreated) const;
};
