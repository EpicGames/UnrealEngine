// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "LiveLinkSourceSettings.h"
#include "LiveLinkOpenVRTypes.generated.h"


/** Settings relevant to both initial factory/"connection" as well as existing sources. */
USTRUCT()
struct FLiveLinkOpenVRCommonSettings
{
	GENERATED_BODY()

public:
	/** Update rate (in Hz) at which to read the tracking data for each device */
	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings", meta=(ClampMin=1, ClampMax=1000))
	uint32 LocalUpdateRateInHz = 60;
};


USTRUCT()
struct FLiveLinkOpenVRConnectionSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings", meta=(ShowOnlyInnerProperties))
	FLiveLinkOpenVRCommonSettings CommonSettings;

	/** Track all tracker pucks */
	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings")
	bool bTrackTrackers = true;

	/** Track all tracking references (e.g. base stations) */
	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings")
	bool bTrackTrackingReferences = true;

	/** Track all controllers */
	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings")
	bool bTrackControllers = false;

	/** Track all HMDs */
	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings")
	bool bTrackHMDs = false;
};


UCLASS()
class ULiveLinkOpenVRSourceSettings : public ULiveLinkSourceSettings
{
	GENERATED_BODY()

public:
	UPROPERTY(EditAnywhere, Category="LiveLinkOpenVR Settings", meta=(ShowOnlyInnerProperties))
	FLiveLinkOpenVRCommonSettings CommonSettings;
};
